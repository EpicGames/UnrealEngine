{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mover Examples",
	"Description": "Non-essential classes and content for the Mover plugin. Includes sample code, test maps, etc to help developers start using the system. Not intended to be used directly in a shipping product. \nPlease refer to the Mover plugin's README document for information about getting started, an overview of concepts, and known issues.",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MoverExamples",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Mover",
			"Enabled": true
		},
		{
			"Name": "ChaosMover",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "CableComponent",
			"Enabled": true
		}
	]
}