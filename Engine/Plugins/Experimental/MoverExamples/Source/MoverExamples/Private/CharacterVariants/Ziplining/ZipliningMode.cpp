// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterVariants/Ziplining/ZipliningMode.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"
#include "CharacterVariants/Ziplining/ZiplineInterface.h"
#include "CharacterVariants/Ziplining/ZipliningTransitions.h"
#include "Kismet/KismetSystemLibrary.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoverLog.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ZipliningMode)





// FZipliningState //////////////////////////////

FMoverDataStructBase* FZipliningState::Clone() const
{
	FZipliningState* CopyPtr = new FZipliningState(*this);
	return CopyPtr;
}

bool FZipliningState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << ZiplineActor;
	Ar.SerializeBits(&bIsMovingAtoB,1);

	bOutSuccess = true;
	return true;
}

void FZipliningState::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("ZiplineActor: %s\n", *GetNameSafe(ZiplineActor));
	Out.Appendf("IsMovingAtoB: %d\n", bIsMovingAtoB);
}

bool FZipliningState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FZipliningState* AuthorityZiplineState = static_cast<const FZipliningState*>(&AuthorityState);

	return (ZiplineActor != AuthorityZiplineState->ZiplineActor) ||
		   (bIsMovingAtoB != AuthorityZiplineState->bIsMovingAtoB);
}

void FZipliningState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FZipliningState* FromState = static_cast<const FZipliningState*>(&From);
	const FZipliningState* ToState = static_cast<const FZipliningState*>(&To);

	ZiplineActor = ToState->ZiplineActor;
	bIsMovingAtoB = ToState->bIsMovingAtoB;
}



// UZipliningMode //////////////////////////////

UZipliningMode::UZipliningMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Transitions.Add(CreateDefaultSubobject<UZiplineEndTransition>(TEXT("ZiplineEndTransition")));
}


void UZipliningMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	UMoverComponent* MoverComp = GetMoverComponent();

	// Ziplining is just following a path from A to B, so all movement is handled in OnSimulationTick
	OutProposedMove = FProposedMove();

}

void UZipliningMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	// Are we continuing a move or starting fresh?
	const FZipliningState* StartingZipState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FZipliningState>();

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	FZipliningState& OutZipState            = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FZipliningState>();

	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	UMoverComponent* MoverComp = Params.MovingComps.MoverComponent.Get();
	AActor* MoverActor = MoverComp->GetOwner();

	USceneComponent* StartPoint = nullptr;
	USceneComponent* EndPoint = nullptr;
	FVector ZipDirection;
	FVector FlatFacingDir;

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	FVector ActorOrigin;
	FVector BoxExtent;
	MoverActor->GetActorBounds(true, OUT ActorOrigin, OUT BoxExtent);
	const FVector ActorToZiplineOffset = MoverComp->GetUpDirection() * BoxExtent.Z;

	if (!StartingZipState)
	{
		// There is no existing zipline state... so let's find the target
		//    A) teleport to the closest starting point, set the zip direction
		//    B) choose the appropriate facing direction
		//    C) choose the appropriate initial velocity
		TArray<AActor*> OverlappingActors;
		MoverComp->GetOwner()->GetOverlappingActors(OUT OverlappingActors);

		for (AActor* CandidateActor : OverlappingActors)
		{
			bool bIsZipline = UKismetSystemLibrary::DoesImplementInterface(CandidateActor, UZipline::StaticClass());

			if (bIsZipline)
			{
				const FVector MoverLoc = UpdatedComponent->GetComponentLocation();
				USceneComponent* ZipPointA = IZipline::Execute_GetStartComponent(CandidateActor);
				USceneComponent* ZipPointB = IZipline::Execute_GetEndComponent(CandidateActor);

				if (FVector::DistSquared(ZipPointA->GetComponentLocation(), MoverLoc) < FVector::DistSquared(ZipPointB->GetComponentLocation(), MoverLoc))
				{
					OutZipState.bIsMovingAtoB = true;
					StartPoint = ZipPointA;
					EndPoint = ZipPointB;
				}
				else
				{
					OutZipState.bIsMovingAtoB = false;
					StartPoint = ZipPointB;
					EndPoint = ZipPointA;
				}

				ZipDirection = (EndPoint->GetComponentLocation() - StartPoint->GetComponentLocation()).GetSafeNormal();

				const FVector WarpLocation = StartPoint->GetComponentLocation() - ActorToZiplineOffset;

				FlatFacingDir = FVector::VectorPlaneProject(ZipDirection, MoverComp->GetUpDirection()).GetSafeNormal();

				OutZipState.ZiplineActor = CandidateActor;

				UpdatedComponent->GetOwner()->TeleportTo(WarpLocation, FlatFacingDir.ToOrientationRotator());

				break;
			}
		}

		// If we were unable to find a valid target zipline, refund all the time and let the actor fall
		if (!StartPoint || !EndPoint)
		{
			FName DefaultAirMode = DefaultModeNames::Falling;
			if (UCommonLegacyMovementSettings* LegacySettings = MoverComp->FindSharedSettings_Mutable<UCommonLegacyMovementSettings>())
			{
				DefaultAirMode = LegacySettings->AirMovementModeName;
			}

			OutputState.MovementEndState.NextModeName = DefaultAirMode;
			OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
			return;
		}

	}
	else
	{
		check(StartingZipState->ZiplineActor);
		OutZipState = *StartingZipState;

		USceneComponent* ZipPointA = IZipline::Execute_GetStartComponent(StartingZipState->ZiplineActor);
		USceneComponent* ZipPointB = IZipline::Execute_GetEndComponent(StartingZipState->ZiplineActor);

		if (StartingZipState->bIsMovingAtoB)
		{
			StartPoint = ZipPointA;
			EndPoint = ZipPointB;
		}
		else
		{
			StartPoint = ZipPointB;
			EndPoint = ZipPointA;
		}

		ZipDirection = (EndPoint->GetComponentLocation() - StartPoint->GetComponentLocation()).GetSafeNormal();
		FlatFacingDir = FVector::VectorPlaneProject(ZipDirection, MoverComp->GetUpDirection()).GetSafeNormal();
	}


	// Now let's slide along the zipline
	const FVector StepStartPos = UpdatedComponent->GetComponentLocation() + ActorToZiplineOffset;
	const FVector DesiredEndPos = StepStartPos + (ZipDirection * MaxSpeed * DeltaSeconds);	// TODO: Make speed more dynamic

	FVector ActualEndPos = FMath::ClosestPointOnSegment(DesiredEndPos,
		StartPoint->GetComponentLocation(),
		EndPoint->GetComponentLocation());

	bool bWillReachEndPosition = (ActualEndPos - EndPoint->GetComponentLocation()).IsNearlyZero();

	FVector MoveDelta = ActualEndPos - StepStartPos;



	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);


	if (!MoveDelta.IsNearlyZero())
	{
		
		FHitResult Hit(1.f);

		UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, MoveDelta, FlatFacingDir.ToOrientationQuat(), true, Hit, ETeleportType::None, MoveRecord);
	}


	const FVector FinalLocation = UpdatedComponent->GetComponentLocation();
	const FVector FinalVelocity = MoveRecord.GetRelevantVelocity();

	OutputSyncState.SetTransforms_WorldSpace(FinalLocation,
		UpdatedComponent->GetComponentRotation(),
		FinalVelocity,
		FVector::ZeroVector,
		nullptr); // no movement base

	UpdatedComponent->ComponentVelocity = FinalVelocity;


	if (bWillReachEndPosition)
	{
		FName DefaultAirMode = DefaultModeNames::Falling;
		if (UCommonLegacyMovementSettings* LegacySettings = MoverComp->FindSharedSettings_Mutable<UCommonLegacyMovementSettings>())
		{
			DefaultAirMode = LegacySettings->AirMovementModeName;
		}

		OutputState.MovementEndState.NextModeName = DefaultAirMode;
		// TODO: If we reach the end position early, we should refund the remaining time

	}
}



