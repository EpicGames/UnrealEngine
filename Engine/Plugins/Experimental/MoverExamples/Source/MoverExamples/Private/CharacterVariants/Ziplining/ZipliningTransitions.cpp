// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterVariants/Ziplining/ZipliningTransitions.h"
#include "CharacterVariants/AbilityInputs.h"
#include "CharacterVariants/Ziplining/ZiplineInterface.h"
#include "DefaultMovementSet/CharacterMoverComponent.h"
#include "GameFramework/Actor.h"
#include "Kismet/KismetSystemLibrary.h"


// UZiplineStartTransition //////////////////////////////

UZiplineStartTransition::UZiplineStartTransition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FTransitionEvalResult UZiplineStartTransition::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult = FTransitionEvalResult::NoTransition;

	UCharacterMoverComponent* MoverComp = Cast<UCharacterMoverComponent>(Params.MovingComps.MoverComponent.Get());

	const FMoverSyncState& SyncState = Params.StartState.SyncState;

	if (MoverComp && MoverComp->IsAirborne() && SyncState.MovementMode != ZipliningModeName)
	{
		if (const FMoverExampleAbilityInputs* AbilityInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FMoverExampleAbilityInputs>())
		{
			if (AbilityInputs->bWantsToStartZiplining)
			{
				TArray<AActor*> OverlappingActors;
				MoverComp->GetOwner()->GetOverlappingActors(OUT OverlappingActors);

				for (AActor* CandidateActor : OverlappingActors)
				{
					bool bIsZipline = UKismetSystemLibrary::DoesImplementInterface(CandidateActor, UZipline::StaticClass());

					if (bIsZipline)
					{
						EvalResult.NextMode = ZipliningModeName;
						break;
					}
				}
			}
		}
	}

	return EvalResult;
}



// UZiplineEndTransition //////////////////////////////

UZiplineEndTransition::UZiplineEndTransition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


FTransitionEvalResult UZiplineEndTransition::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	FTransitionEvalResult EvalResult = FTransitionEvalResult::NoTransition;

	if (const FCharacterDefaultInputs* DefaultInputs = Params.StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
	{
		if (DefaultInputs->bIsJumpJustPressed)
		{
			EvalResult.NextMode = AutoExitToMode;
		}
	}

	return EvalResult;
}

void UZiplineEndTransition::Trigger_Implementation(const FSimulationTickParams& Params)
{
	//TODO: create a small jump, using current directionality
}
