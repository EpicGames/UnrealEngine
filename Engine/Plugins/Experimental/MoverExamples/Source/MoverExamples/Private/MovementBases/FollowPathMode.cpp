// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementBases/FollowPathMode.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FollowPathMode)




UFollowPathMode::UFollowPathMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UFollowPathMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	// TODO: Rework to maintain the split between generating and executing moves
}

void UFollowPathMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();

	const FMoverDefaultSyncState* StartingMoveState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	const FFollowPathState* StartingPathState		= StartState.SyncState.SyncStateCollection.FindDataByType<FFollowPathState>();

	FMoverDefaultSyncState& OutputMoveState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	FFollowPathState& OutputPathState		= OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FFollowPathState>();

	FVector StartingLocation = StartingMoveState->GetLocation_WorldSpace();
	FRotator StartingRotation = StartingMoveState->GetOrientation_WorldSpace();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	// If we don't already have a valid pathing state, we need to initialize 
	if (!StartingPathState || !StartingPathState->HasValidPathState())
	{
		// Indicates we haven't started pathing yet. Finalize control points and capture origins
		OutputPathState.BaseLocation = UpdatedComponent->GetComponentLocation();
		OutputPathState.CurrentPathPos = 0.f;
		OutputPathState.CurrentDirectionMod = 1.f;

		if (ControlPoints.Num() > 0)
		{
			FInterpControlPoint& StartPoint = ControlPoints[0];
			StartingLocation = StartPoint.bPositionIsRelative ? OutputPathState.BaseLocation + StartPoint.PositionControlPoint : StartPoint.PositionControlPoint;
		}
		else
		{
			StartingLocation = UpdatedComponent->GetComponentLocation();
		}

		// Move the component to the first path location
		FHitResult IgnoredHit(1.f);
		UpdatedComponent->MoveComponent(StartingLocation - OutputPathState.BaseLocation, UpdatedComponent->GetComponentRotation(), false, &IgnoredHit);
	}

	float RemainingSecs = DeltaSeconds;

	while (RemainingSecs > 0.f)
	{
		float StepSecs = RemainingSecs;
		RemainingSecs -= StepSecs;

		// Compute new time lerp alpha based on how far we moved	
		float DurationPctRemainder = 0.f;
		bool bStopped = false;
		float NewDirectionMod = OutputPathState.CurrentDirectionMod;

		OutputPathState.CurrentPathPos = CalculateNewPathPct(OutputPathState.CurrentPathPos, OutputPathState.CurrentDirectionMod, DeltaSeconds, /* out */ bStopped, /* out */ DurationPctRemainder, /* out */  NewDirectionMod);

		OutputPathState.CurrentDirectionMod = NewDirectionMod;

		// Apply any remaining time
		if (DurationPctRemainder != 0.f && !bStopped)
		{
			RemainingSecs += (DurationPctRemainder * Duration);
		}

		// Compute a move delta to get to that position
		FVector MoveDelta = ComputeMoveDelta(StartingLocation, OutputPathState.BaseLocation, OutputPathState.CurrentPathPos);

		FRotator DesiredOrientation = ComputeMoveOrientation(OutputPathState.CurrentPathPos, OutputPathState.BaseLocation, UpdatedComponent->GetComponentRotation());

		// Move the object
		FHitResult IgnoredHit(1.f);
		UpdatedComponent->MoveComponent(MoveDelta, DesiredOrientation, false, &IgnoredHit);
	}

	// Capture final state
	FVector Velocity = (UpdatedComponent->GetComponentLocation() - StartingLocation) / DeltaSeconds;

	// Compute angular velocity
	FRotator NewRotation = UpdatedComponent->GetComponentRotation();
	FVector AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(StartingRotation, NewRotation, DeltaSeconds);
	
	OutputMoveState.SetTransforms_WorldSpace( UpdatedComponent->GetComponentLocation(),
		                                      NewRotation,
											  Velocity,
											  AngularVelocityDegrees,
											  nullptr ); // no movement base

	UpdatedComponent->ComponentVelocity = Velocity;
}

void UFollowPathMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);
	
	UpdateControlPoints(true);
	TimeMultiplier = 1.f / (Duration > UE_KINDA_SMALL_NUMBER ? Duration : UE_KINDA_SMALL_NUMBER);
}

void UFollowPathMode::UpdateControlPoints(bool InForceUpdate)
{
	const USceneComponent* UpdatedComponent = nullptr;
	const UObject* OwnerObject = GetOuter();


	while (OwnerObject != nullptr)
	{
		if (const AActor* OwnerActor = Cast<const AActor>(OwnerObject))
		{
			UpdatedComponent = OwnerActor->GetRootComponent();
			break;
		}
		else
		{
			OwnerObject = OwnerObject->GetOuter();
		}
	}

	if (UpdatedComponent != nullptr)
	{
		if (InForceUpdate == true)
		{
			FVector BasePosition = UpdatedComponent->GetComponentLocation();

			TotalDistance = 0.0f;
			ControlPointPathTangents.SetNumUninitialized(ControlPoints.Num());

			if (ControlPoints.Num() > 0)
			{
				FVector CurrentPos = ControlPoints[0].PositionControlPoint;
				if (ControlPoints[0].bPositionIsRelative == true)
				{
					CurrentPos += BasePosition;
				}

				// Calculate the distances from point to point
				for (int32 ControlPoint = 0; ControlPoint < ControlPoints.Num(); ControlPoint++)
				{
					if (ControlPoint + 1 < ControlPoints.Num())
					{
						FVector NextPosition = ControlPoints[ControlPoint + 1].PositionControlPoint;
						if (ControlPoints[ControlPoint + 1].bPositionIsRelative == true)
						{
							NextPosition += BasePosition;
						}
						ControlPoints[ControlPoint].DistanceToNext = (NextPosition - CurrentPos).Size();

						TotalDistance += ControlPoints[ControlPoint].DistanceToNext;
						CurrentPos = NextPosition;
					}
					else
					{
						ControlPoints[ControlPoint].DistanceToNext = 0.0f;
						ControlPoints[ControlPoint].Percentage = 1.0f;
						ControlPoints[ControlPoint].StartTime = 1.0f;
					}
				}
				float Percent = 0.0f;
				// Use the distance to determine what % of time to spend going from each point
				for (int32 ControlPoint = 0; ControlPoint < ControlPoints.Num(); ControlPoint++)
				{
					ControlPoints[ControlPoint].StartTime = Percent;
					if (ControlPoints[ControlPoint].DistanceToNext != 0.0f)
					{
						ControlPoints[ControlPoint].Percentage = ControlPoints[ControlPoint].DistanceToNext / TotalDistance;
						Percent += ControlPoints[ControlPoint].Percentage;
					}
				}

				// Calculate the path tangent for each point
				if (ControlPoints.Num() > 1)
				{
					for (int32 i = 0; i < ControlPoints.Num(); ++i)
					{
						if (i == 0)	// Special case: first point only has 1 influence
						{
							ControlPointPathTangents[i] = (ControlPoints[i+1].PositionControlPoint - ControlPoints[i].PositionControlPoint).GetSafeNormal();
						}
						else if (i == ControlPoints.Num() - 1)	// Special case: last point only has 1 influence
						{
							ControlPointPathTangents[i] = (ControlPoints[i].PositionControlPoint - ControlPoints[i-1].PositionControlPoint).GetSafeNormal();
						}
						else
						{
							ControlPointPathTangents[i] = ( (ControlPoints[i].PositionControlPoint - ControlPoints[i-1].PositionControlPoint).GetSafeNormal()
														  + (ControlPoints[i+1].PositionControlPoint - ControlPoints[i].PositionControlPoint).GetSafeNormal() );

							ControlPointPathTangents[i].Normalize();
						}
					}
				}
				else
				{
					ControlPointPathTangents[0] = FVector::ForwardVector;
				}
			}
		}
	}
}


float UFollowPathMode::CalculateNewPathPct(float InPathPct, float InDirectionMod, float InDeltaSecs, bool& OutStopped, float& OutTimeRemainder, float& OutNewDirectionMod) const
{
	OutTimeRemainder = 0.0f;
	float NewPathPct = InPathPct;
	OutStopped = false;
	OutNewDirectionMod = InDirectionMod;

	NewPathPct += ((InDeltaSecs * TimeMultiplier) * InDirectionMod);
	if (NewPathPct >= 1.0f)
	{
		OutTimeRemainder = NewPathPct - 1.0f;
		if (BehaviourType == EInterpToBehaviourType::OneShot)
		{
			NewPathPct = 1.0f;
			OutStopped = true;
		}
		else if (BehaviourType == EInterpToBehaviourType::Loop_Reset)
		{
			NewPathPct = 0.0f;
		}
		else  // PingPong: reverse direction
		{
			OutNewDirectionMod = -InDirectionMod;
			NewPathPct = 1.0f;
		}
	}
	else if (NewPathPct < 0.0f)
	{
		OutTimeRemainder = -NewPathPct;
		if (BehaviourType == EInterpToBehaviourType::OneShot_Reverse)
		{
			NewPathPct = 0.0f;
			OutStopped = true;
		}
		else if (BehaviourType == EInterpToBehaviourType::PingPong)
		{
			OutNewDirectionMod = -InDirectionMod;
			NewPathPct = 0.0f;
		}
	}

	return NewPathPct;
}



FVector UFollowPathMode::ComputeMoveDelta(const FVector CurrentPos, const FVector BaseLocation, const float TargetPathPct) const
{
	FVector MoveDelta = FVector::ZeroVector;
	FVector NewPosition = CurrentPos;
	//Find current control point
	float PathPct = 0.0f;
	int32 CurrentControlPoint = INDEX_NONE;
	// Always use the end point if we are at the end 
	if (TargetPathPct >= 1.0f)
	{
		CurrentControlPoint = ControlPoints.Num() - 1;
	}
	else
	{
		for (int32 iSpline = 0; iSpline < ControlPoints.Num(); iSpline++)
		{
			float NextTime = PathPct + ControlPoints[iSpline].Percentage;
			if (TargetPathPct < NextTime)
			{
				CurrentControlPoint = iSpline;
				break;
			}
			PathPct = NextTime;
		}
	}
	// If we found a valid control point get the position between it and the next
	if (CurrentControlPoint != INDEX_NONE)
	{
		float Base = TargetPathPct - ControlPoints[CurrentControlPoint].StartTime;
		float ThisAlpha = Base / ControlPoints[CurrentControlPoint].Percentage;
		FVector BeginControlPoint = ControlPoints[CurrentControlPoint].PositionControlPoint + (ControlPoints[CurrentControlPoint].bPositionIsRelative ? BaseLocation : FVector::ZeroVector);

		int32 NextControlPoint = FMath::Clamp(CurrentControlPoint + 1, 0, ControlPoints.Num() - 1);
		FVector EndControlPoint = ControlPoints[NextControlPoint].PositionControlPoint;
		EndControlPoint = ControlPoints[NextControlPoint].PositionControlPoint + (ControlPoints[NextControlPoint].bPositionIsRelative ? BaseLocation : FVector::ZeroVector);

		NewPosition = FMath::Lerp(BeginControlPoint, EndControlPoint, ThisAlpha);
	}

	if (CurrentPos != NewPosition)
	{
		MoveDelta = NewPosition - CurrentPos;
	}
	return MoveDelta;

}

FRotator UFollowPathMode::ComputeMoveOrientation(const float TargetPathPos, const FVector& BaseLocation, FRotator DefaultOrientation) const
{
	FRotator ReturnOrientation;

	if (RotationType == EFollowPathRotationType::AlignWithPathTangents)
	{
		ReturnOrientation = ComputeInterpolatedTangentFromPathPct(TargetPathPos).ToOrientationRotator();
	}
	else if (RotationType == EFollowPathRotationType::AlignWithPath)
	{
		ReturnOrientation = ComputeTangentFromPathPct(TargetPathPos, BaseLocation).ToOrientationRotator();
	}
	else if (RotationType == EFollowPathRotationType::Fixed)
	{
		ReturnOrientation = DefaultOrientation;
	}
	else
	{
		UE_LOGF(LogMover, Warning, "EFollowPathRotationType %i is not supported yet. Using default orietation instead", int(RotationType));
		ReturnOrientation = DefaultOrientation;
	}

	return ReturnOrientation;
}

FVector UFollowPathMode::ComputeInterpolatedTangentFromPathPct(const float PathPct) const
{
	if (ControlPoints.IsEmpty())
	{
		return FVector::ForwardVector;
	}

	FVector InfluenceTangentA(FVector::ForwardVector), InfluenceTangentB(FVector::ForwardVector);
	float InfluencePctA(0.5f), InfluencePctB(0.5f);

	if (PathPct <= 0.f || ControlPoints.Num() == 1)
	{
		InfluenceTangentA = InfluenceTangentB = ControlPointPathTangents[0];
		InfluencePctA = InfluencePctB = ControlPoints[0].Percentage;

	}
	else if (PathPct >= 1.f)
	{
		InfluenceTangentA = InfluenceTangentB = ControlPointPathTangents[ControlPoints.Num() - 1];
		InfluencePctA = InfluencePctB = ControlPoints[ControlPoints.Num() - 1].Percentage;

	}
	else
	{
		for (int32 i = 0; i < ControlPoints.Num() - 1; ++i)
		{
			if (PathPct < ControlPoints[i+1].StartTime)
			{
				InfluenceTangentA = ControlPointPathTangents[i];
				InfluencePctA = ControlPoints[i].StartTime;

				InfluenceTangentB = ControlPointPathTangents[i+1];
				InfluencePctB = ControlPoints[i+1].StartTime;
				break;
			}
		}
	}


	// Get the weighted average between influences
	float TotalWeight = InfluencePctB - InfluencePctA;

	if (FMath::IsNearlyZero(TotalWeight))
	{
		return (InfluenceTangentA + InfluenceTangentB).GetSafeNormal();
	}

	FVector InterpolatedTangent = (InfluenceTangentA * (1.f - (FMath::Abs(InfluencePctA - PathPct) / TotalWeight)))
								+ (InfluenceTangentB * (1.f - (FMath::Abs(InfluencePctB - PathPct) / TotalWeight)));

	return InterpolatedTangent.GetSafeNormal();
}

FVector UFollowPathMode::ComputeTangentFromPathPct(const float PathPct, const FVector& BaseLocation) const
{
	if (ControlPoints.Num() > 1)
	{
		float PathPctAtNextPoint = 0.f;

		for (int32 i=0; i < ControlPoints.Num()-1; ++i)
		{
			PathPctAtNextPoint += ControlPoints[i].Percentage;
			if (PathPct < PathPctAtNextPoint)
			{
				const FVector FromPos = ControlPoints[i].PositionControlPoint + (ControlPoints[i].bPositionIsRelative ? BaseLocation : FVector::ZeroVector);
				const FVector ToPos   = ControlPoints[i+1].PositionControlPoint + (ControlPoints[i+1].bPositionIsRelative ? BaseLocation : FVector::ZeroVector);

				return (ToPos-FromPos).GetSafeNormal();
			}
		}
	}
	return FVector::ForwardVector;
}


#if WITH_EDITOR
void UFollowPathMode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (ControlPoints.Num() != 0)
	{
		UpdateControlPoints(true);
	}
}


#endif // WITH_EDITOR




FMoverDataStructBase* FFollowPathState::Clone() const
{
	FFollowPathState* CopyPtr = new FFollowPathState(*this);
	return CopyPtr;
}
