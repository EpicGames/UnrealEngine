// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementBases/FollowSplineMode.h"

#include "MoverComponent.h"
#include "MoveLibrary/MovementUtils.h"
#include "Components/SplineComponent.h"
#include "Curves/CurveFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FollowSplineMode)

namespace FollowSplineMode::Utils::Private
{
	static float ComputeRangeInputValue(const USplineComponent* ControlSpline, FSplineOffsetRangeInput RangeOffsetInput)
	{
		switch (RangeOffsetInput.OffsetUnit)
		{
		case ESplineOffsetUnit::Percentage:
		{
			return FMath::Clamp(RangeOffsetInput.Value, 0.0f, 1.0f) * ControlSpline->Duration; 
		}
		case ESplineOffsetUnit::DurationAbsoluteSeconds:
		{
			return FMath::Clamp(RangeOffsetInput.Value, 0.0f, ControlSpline->Duration);
		}
		case ESplineOffsetUnit::DistanceAbsolute:
		{
			const float SplineTime = ControlSpline->GetTimeAtDistanceAlongSpline(RangeOffsetInput.Value);
			return FMath::Clamp(SplineTime, 0.0f, ControlSpline->Duration);
		}
		default:
		{
			UE_LOGF(LogMover, Warning, "Unknown Spline Offset Unit");
			break;
		}
		}
		return 0.0f;
	}
}


UFollowSplineMode::UFollowSplineMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	BehaviourType(EInterpToBehaviourType::OneShot),
	RotationType(EFollowSplineRotationType::FollowSplineTangent),
	StartOffset({.Value = 0.0f, .OffsetUnit = ESplineOffsetUnit::Percentage}),
	EndOffset({.Value = 1.0f, .OffsetUnit = ESplineOffsetUnit::Percentage })
{
}

void UFollowSplineMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	// TODO: Rework to maintain the split between generating and executing moves
}

void UFollowSplineMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	const FMoverTickStartData& StartState = Params.StartState;
	const FMovingComponentSet& MovingComps = Params.MovingComps;

	const FMoverDefaultSyncState* StartingMoveState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	const FFollowSplineState* StartingPathState		= StartState.SyncState.SyncStateCollection.FindDataByType<FFollowSplineState>();

	FMoverDefaultSyncState& OutputMoveState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	FFollowSplineState& OutputPathState		= OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FFollowSplineState>();

	FVector StartingLocation = StartingMoveState->GetLocation_WorldSpace();
	FRotator StartingOrientation = StartingMoveState->GetOrientation_WorldSpace();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	if (!ControlSpline)
	{
		return;
	}

	// Initilization
	StartReversedMultiplier = StartReveresed ? -1 : 1;
	StartOffsetSeconds = FollowSplineMode::Utils::Private::ComputeRangeInputValue(ControlSpline, StartOffset);
	EndOffsetSeconds = FollowSplineMode::Utils::Private::ComputeRangeInputValue(ControlSpline, EndOffset);

	// Using strictly less than to avoid divide by zero exceptions. Otherwise FollowDuration would become zero.
	if (!ensureMsgf(StartOffsetSeconds < EndOffsetSeconds, TEXT("StartOffset should be less than EndOffset. To reverse the direction use the bSplineFollowDirection property. Aborting.")))
	{
		return;
	}

	FollowDuration = EndOffsetSeconds - StartOffsetSeconds;
	// Retrieve the data from the SplineState
	if (!StartingPathState || 
		(StartingPathState && StartingPathState->CurrentSplineTime == -1.0f)) // Uninitialized, use the current offset if set.
	{
		CurrentSplineTime +=  DeltaSeconds;
	}
	else
	{
		CurrentSplineTime = StartingPathState->CurrentSplineTime + DeltaSeconds;
		FollowDirectionMultiplier = StartingPathState->CurrentDirectionMultiplier;
	}
		
	float MeasuredSplineTime = CurrentSplineTime;

	// Apply Interpolation Curve based speed control
	if (InterpolationCurve)
	{
		const float CurrentSplinePct = FMath::Clamp(MeasuredSplineTime / FollowDuration, 0.0f, 1.0f);
		MeasuredSplineTime = FollowDuration * FMath::Clamp(InterpolationCurve->GetFloatValue(CurrentSplinePct), 0.0f,1.0f);
	}
	
	// Apply custom duration for follow motion
	float CustomFollowDuration = FollowDuration;
	if (CustomDurationSecondsOverride > 0.0f)
	{
		CustomFollowDuration = CustomDurationSecondsOverride;
	}

	// Move the relative time to offset time frame.
	const float MappedSplineTime = FMath::GetMappedRangeValueClamped(
		FVector2f(0.0f, 2.0f * CustomFollowDuration),
		FVector2f(StartOffsetSeconds, EndOffsetSeconds + FollowDuration), 
		MeasuredSplineTime);


	if (CanMove(MappedSplineTime))
	{
		// Move the object
		const FTransform SplineTransform = GetTransformAtTime(MappedSplineTime, MovingComps.UpdatedComponent->GetComponentRotation());
		const FVector MoveDelta = SplineTransform.GetLocation() - StartingLocation;
		const FVector Velocity = DeltaSeconds > UE_SMALL_NUMBER ? MoveDelta / DeltaSeconds : FVector::ZeroVector;

		FMovementRecord& MoveRecord = OutputState.MoveRecord;
		MoveRecord.SetDeltaSeconds(DeltaSeconds);
		
		FHitResult MoveHitResult;
		
		UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps, MoveDelta, SplineTransform.GetRotation(), true, MoveHitResult, ETeleportType::None, MoveRecord);

		UpdatePathState(OutputPathState);
		FRotator NewRotation = MovingComps.UpdatedComponent->GetComponentRotation();
		FVector AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(StartingOrientation, NewRotation, DeltaSeconds);
		
		// Update Move State
		OutputMoveState.SetTransforms_WorldSpace(MovingComps.UpdatedComponent->GetComponentLocation(),
			NewRotation,
			Velocity,
			AngularVelocityDegrees,
			nullptr); // no movement base
	}
}

void UFollowSplineMode::SetControlSpline(const AActor* SplineProviderActor, FSplineOffsetRangeInput Offset)
{
	if (SplineProviderActor)
	{
		if (USplineComponent* SplineComponent = SplineProviderActor->GetComponentByClass<USplineComponent>())
		{
			using namespace FollowSplineMode::Utils::Private;
			ControlSpline = SplineComponent;
		
			const float StartOffsetTime = ComputeRangeInputValue(ControlSpline, StartOffset);
			const float EndOffsetTime = ComputeRangeInputValue(ControlSpline, EndOffset);
			const float InitialOffsetTime = ComputeRangeInputValue(ControlSpline, Offset);

			CurrentSplineTime = FMath::Clamp(InitialOffsetTime, StartOffsetTime, EndOffsetTime) - StartOffsetTime;
		}
	}
}

void UFollowSplineMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	ConfigureSplineData();
}

void UFollowSplineMode::ConfigureSplineData()
{
	// Control Spline is already set
	if (ControlSpline)
	{
		return;
	}

	// Attempt to find a spline component on the actor who owns this mode
	const UObject* OwnerObject = GetOuter();
	while (OwnerObject != nullptr)
	{
		if (const AActor* OwnerActor = Cast<const AActor>(OwnerObject))
		{
			SetControlSpline(OwnerActor);
			break;
		}
		else
		{
			OwnerObject = OwnerObject->GetOuter();
		}
	}
}

FTransform UFollowSplineMode::GetTransformAtTime(float MeasuredSplineTime, const FRotator& DefaultOrientation)
{
	const float BehaviorSplineTime = ApplyBehaviorType(MeasuredSplineTime);
	const float DirectionSplineTime = ApplyFollowDirection(BehaviorSplineTime);

	FTransform SplineTransform = ControlSpline->GetTransformAtTime(DirectionSplineTime, ESplineCoordinateSpace::World, bConstantFollowVelocity);
	if (RotationType == EFollowSplineRotationType::NoRotation)
	{
		const FQuat RotationQuat(DefaultOrientation);
		SplineTransform.SetRotation(RotationQuat);
	}
	else if (bOrientMoverToMovement)
	{
		const FVector Tangent = FMath::Sign(OrientationMultiplier * FollowDirectionMultiplier * StartReversedMultiplier) * ControlSpline->GetTangentAtTime(DirectionSplineTime, ESplineCoordinateSpace::World, bConstantFollowVelocity);
		const FQuat Orientation = Tangent.ToOrientationQuat();
		SplineTransform.SetRotation(Orientation);
	}

	return SplineTransform;
}

float UFollowSplineMode::ApplyBehaviorType(float MeasuredSplineTime)
{
	switch (BehaviourType)
	{
	case EInterpToBehaviourType::OneShot_Reverse:
	{
		MeasuredSplineTime = EndOffsetSeconds - MeasuredSplineTime + StartOffsetSeconds;
		if (MeasuredSplineTime >= EndOffsetSeconds)
		{
			MeasuredSplineTime = EndOffsetSeconds;
		}

		if (bOrientMoverToMovement)
		{
			OrientationMultiplier = -1;
		}
		else
		{
			OrientationMultiplier = 1;
		}
		break;
	}
	case EInterpToBehaviourType::Loop_Reset:
	{
		if (MeasuredSplineTime >= EndOffsetSeconds)
		{
			MeasuredSplineTime = StartOffsetSeconds;
			CurrentSplineTime = 0.0f;
		}
		break;
	}
	case EInterpToBehaviourType::PingPong:
	{
		if (MeasuredSplineTime >= EndOffsetSeconds)
		{
			FollowDirectionMultiplier = -1;
		}

		if (MeasuredSplineTime >= EndOffsetSeconds + FollowDuration)
		{
			bResetPingPong = true;
		}
	}
	case EInterpToBehaviourType::OneShot:
		// falls through
	default:
		break;
	}

	return MeasuredSplineTime;
}

float UFollowSplineMode::ApplyFollowDirection(float MeasuredSplineTime)
{
	// Wrap the time around as ping pong mode maps the motion to twice the duration
	while (MeasuredSplineTime < StartOffsetSeconds)
	{
		MeasuredSplineTime += FollowDuration;
	}
	while (MeasuredSplineTime > EndOffsetSeconds)
	{
		MeasuredSplineTime -= FollowDuration;
	}

	// Apply the final directional inversion
	if (FMath::Sign(FollowDirectionMultiplier * StartReversedMultiplier) < 0)
	{
		MeasuredSplineTime = EndOffsetSeconds - MeasuredSplineTime + StartOffsetSeconds;
	}
	
	return MeasuredSplineTime;
}

void UFollowSplineMode::UpdatePathState(FFollowSplineState& OutputPathState)
{
	if (bResetPingPong)
	{
		OutputPathState.CurrentSplineTime = 0.0f;
		OutputPathState.CurrentDirectionMultiplier = 1;	
		bResetPingPong = false;
	}
	else
	{
		OutputPathState.CurrentSplineTime = CurrentSplineTime;
		OutputPathState.CurrentDirectionMultiplier = FollowDirectionMultiplier;
	}
}

bool UFollowSplineMode::CanMove(float MeasuredSplineTime) const
{
	switch (BehaviourType)
	{
	case EInterpToBehaviourType::Loop_Reset:
		// falls through
	case EInterpToBehaviourType::PingPong:
	{
		return true;
	}
	case EInterpToBehaviourType::OneShot_Reverse:
		// falls through
	case EInterpToBehaviourType::OneShot:
		// falls through
	default:
		break;
	}

	return  (StartOffsetSeconds <= MeasuredSplineTime) && (MeasuredSplineTime <= EndOffsetSeconds);
}

FMoverDataStructBase* FFollowSplineState::Clone() const
{
	FFollowSplineState* CopyPtr = new FFollowSplineState(*this);
	return CopyPtr;
}
