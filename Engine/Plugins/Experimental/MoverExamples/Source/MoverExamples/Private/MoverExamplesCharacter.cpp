// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoverExamplesCharacter.h"
#include "Components/InputComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/LocalPlayer.h"
#include "DefaultMovementSet/CharacterMoverComponent.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "CharacterVariants/AbilityInputs.h"
#include "GameFramework/SpringArmComponent.h"
#include "GameFramework/Controller.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/PhysicsVolume.h"
#include "EnhancedInputComponent.h"
#include "InputAction.h"
#include "DefaultMovementSet/NavMoverComponent.h"

AMoverExamplesCharacter::AMoverExamplesCharacter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NavMoverComponent(nullptr)
{
	PrimaryActorTick.bCanEverTick = true;

	SetReplicatingMovement(false);	// disable Actor-level movement replication, since our Mover component will handle it

	auto IsImplementedInBlueprint = [](const UFunction* Func) -> bool
	{
		return Func && ensure(Func->GetOuter())
			&& Func->GetOuter()->IsA(UBlueprintGeneratedClass::StaticClass());
	};

	static FName ProduceInputBPFuncName = FName(TEXT("OnProduceInputInBlueprint"));
	UFunction* ProduceInputFunction = GetClass()->FindFunctionByName(ProduceInputBPFuncName);
	bHasProduceInputinBpFunc = IsImplementedInBlueprint(ProduceInputFunction);
}

void AMoverExamplesCharacter::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	CharacterMotionComponent = FindComponentByClass<UCharacterMoverComponent>();

	if (CharacterMotionComponent)
	{
		if (USceneComponent* UpdatedComponent = CharacterMotionComponent->GetUpdatedComponent())
		{
			UpdatedComponent->SetCanEverAffectNavigation(bCanAffectNavigationGeneration);
		}
	}
}

// Called every frame
void AMoverExamplesCharacter::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	// Do whatever you want here. By now we have the latest movement state and latest input processed.


	// Spin camera based on input
	if (APlayerController* PC = Cast<APlayerController>(GetController()))
	{
		// Simple input scaling. A real game will probably map this to an acceleration curve
		static float LookRateYaw = 100.f;	// degs/sec
		static float LookRatePitch = 100.f;	// degs/sec

		PC->AddYawInput(CachedLookInput.Yaw * LookRateYaw * DeltaTime);
		PC->AddPitchInput(-CachedLookInput.Pitch * LookRatePitch * DeltaTime);
	}

	// Clear all camera-related cached input
	CachedLookInput = FRotator::ZeroRotator;
}

void AMoverExamplesCharacter::BeginPlay()
{
	Super::BeginPlay();

	if (APlayerController* PC = Cast<APlayerController>(GetController()))
	{
		PC->PlayerCameraManager->ViewPitchMax = 89.0f;
		PC->PlayerCameraManager->ViewPitchMin = -89.0f;
	}
	
	NavMoverComponent = FindComponentByClass<UNavMoverComponent>();
}

// Called to bind functionality to input
void AMoverExamplesCharacter::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	Super::SetupPlayerInputComponent(PlayerInputComponent);

	// Setup some bindings - we are currently using Enhanced Input and just using some input actions assigned in editor for simplicity
	if (UEnhancedInputComponent* Input = Cast<UEnhancedInputComponent>(PlayerInputComponent))
	{	
		Input->BindAction(MoveInputAction, ETriggerEvent::Triggered, this, &AMoverExamplesCharacter::OnMoveTriggered);
		Input->BindAction(MoveInputAction, ETriggerEvent::Completed, this, &AMoverExamplesCharacter::OnMoveCompleted);
		Input->BindAction(LookInputAction, ETriggerEvent::Triggered, this, &AMoverExamplesCharacter::OnLookTriggered);
		Input->BindAction(LookInputAction, ETriggerEvent::Completed, this, &AMoverExamplesCharacter::OnLookCompleted);
		Input->BindAction(JumpInputAction, ETriggerEvent::Started, this, &AMoverExamplesCharacter::OnJumpStarted);
		Input->BindAction(JumpInputAction, ETriggerEvent::Completed, this, &AMoverExamplesCharacter::OnJumpReleased);
		Input->BindAction(FlyInputAction, ETriggerEvent::Triggered, this, &AMoverExamplesCharacter::OnFlyTriggered);
	}
}


FVector AMoverExamplesCharacter::GetNavAgentLocation() const
{
	FVector AgentLocation = FNavigationSystem::InvalidLocation;
	const USceneComponent* UpdatedComponent = CharacterMotionComponent ? CharacterMotionComponent->GetUpdatedComponent() : nullptr;
	
	if (NavMoverComponent)
	{
		AgentLocation = NavMoverComponent->GetFeetLocation();
	}
	
	if (FNavigationSystem::IsValidLocation(AgentLocation) == false && UpdatedComponent != nullptr)
	{
		AgentLocation = UpdatedComponent->GetComponentLocation() - FVector(0,0,UpdatedComponent->Bounds.BoxExtent.Z);
	}

	return AgentLocation;
}

void AMoverExamplesCharacter::UpdateNavigationRelevance()
{
	if (CharacterMotionComponent)
	{
		if (USceneComponent* UpdatedComponent = CharacterMotionComponent->GetUpdatedComponent())
		{
			UpdatedComponent->SetCanEverAffectNavigation(bCanAffectNavigationGeneration);
		}
	}
}

void AMoverExamplesCharacter::ProduceInput_Implementation(int32 SimTimeMs, FMoverInputCmdContext& InputCmdResult)
{
	OnProduceInput((float)SimTimeMs, InputCmdResult);

	if (bHasProduceInputinBpFunc)
	{
		InputCmdResult = OnProduceInputInBlueprint((float)SimTimeMs, InputCmdResult);
	}
}


void AMoverExamplesCharacter::OnProduceInput(float DeltaMs, FMoverInputCmdContext& OutInputCmd)
{

	// Generate user commands. Called right before the Character movement simulation will tick (for a locally controlled pawn)
	// This isn't meant to be the best way of doing a camera system. It is just meant to show a couple of ways it may be done
	// and to make sure we can keep distinct the movement, rotation, and view angles.
	// Styles 1-3 are really meant to be used with a gamepad.
	//
	// Its worth calling out: the code that happens here is happening *outside* of the Character movement simulation. All we are doing
	// is generating the input being fed into that simulation. That said, this means that A) the code below does not run on the server
	// (and non controlling clients) and B) the code is not rerun during reconcile/resimulates. Use this information guide any
	// decisions about where something should go (such as aim assist, lock on targeting systems, etc): it is hard to give absolute
	// answers and will depend on the game and its specific needs. In general, at this time, I'd recommend aim assist and lock on 
	// targeting systems to happen /outside/ of the system, i.e, here. But I can think of scenarios where that may not be ideal too.

	FCharacterDefaultInputs& CharacterInputs = OutInputCmd.InputCollection.FindOrAddMutableDataByType<FCharacterDefaultInputs>();

	if (GetController() == nullptr)
	{
		if (GetLocalRole() == ENetRole::ROLE_Authority && GetRemoteRole() == ENetRole::ROLE_SimulatedProxy)
		{
			static const FCharacterDefaultInputs DoNothingInput;
			// If we get here, that means this pawn is not currently possessed and we're choosing to provide default do-nothing input
			CharacterInputs = DoNothingInput;
		}

		// We don't have a local controller so we can't run the code below. This is ok. Simulated proxies will just use previous input when extrapolating
		return;
	}

	if (USpringArmComponent* SpringComp = FindComponentByClass<USpringArmComponent>())
	{
		// This is not best practice: do not search for component every frame
		SpringComp->bUsePawnControlRotation = true;
	}

	CharacterInputs.ControlRotation = FRotator::ZeroRotator;

	APlayerController* PC = Cast<APlayerController>(GetController());
	if (PC)
	{
		CharacterInputs.ControlRotation = PC->GetControlRotation();
	}

	bool bRequestedNavMovement = false;
	if (NavMoverComponent)
	{
		bRequestedNavMovement = NavMoverComponent->ConsumeNavMovementData(CachedMoveInputIntent, CachedMoveInputVelocity);
	}
	
	// Favor velocity input 
	bool bUsingInputIntentForMove = CachedMoveInputVelocity.IsZero();

	if (bUsingInputIntentForMove)
	{
		const FVector FinalDirectionalIntent = CharacterInputs.ControlRotation.RotateVector(CachedMoveInputIntent);
		CharacterInputs.SetMoveInput(EMoveInputType::DirectionalIntent, FinalDirectionalIntent);
	}
	else
	{
		CharacterInputs.SetMoveInput(EMoveInputType::Velocity, CachedMoveInputVelocity);
	}

	// Normally cached input is cleared by OnMoveCompleted input event but that won't be called if movement came from nav movement
	if (bRequestedNavMovement)
	{
		CachedMoveInputIntent = FVector::ZeroVector;
		CachedMoveInputVelocity = FVector::ZeroVector;
	}
	
	static float RotationMagMin(1e-3f);

	const bool bHasAffirmativeMoveInput = (CharacterInputs.GetMoveInput().Size() >= RotationMagMin);
	
	// Figure out intended orientation
	CharacterInputs.OrientationIntent = FVector::ZeroVector;


	if (bHasAffirmativeMoveInput)
	{
		if (bOrientRotationToMovement)
		{
			// set the intent to the actors movement direction
			CharacterInputs.OrientationIntent = CharacterInputs.GetMoveInput().GetSafeNormal();
		}
		else
		{
			// set intent to the the control rotation - often a player's camera rotation
			CharacterInputs.OrientationIntent = CharacterInputs.ControlRotation.Vector().GetSafeNormal();
		}

		LastAffirmativeOrientationIntent = CharacterInputs.OrientationIntent;
	}
	else if (bMaintainLastInputOrientation)
	{
		// There is no movement intent, so use the last-known affirmative move input
		CharacterInputs.OrientationIntent = LastAffirmativeOrientationIntent;
	}
	
	CharacterInputs.bIsJumpPressed = bIsJumpPressed;
	CharacterInputs.bIsJumpJustPressed = bIsJumpJustPressed;

	if (bShouldToggleFlying)
	{
		if (!bIsFlyingActive)
		{
			CharacterInputs.SuggestedMovementMode = DefaultModeNames::Flying;
		}
		else
		{
			CharacterInputs.SuggestedMovementMode = DefaultModeNames::Falling;
		}

		bIsFlyingActive = !bIsFlyingActive;
	}
	else
	{
		CharacterInputs.SuggestedMovementMode = NAME_None;
	}

	// Convert inputs to be relative to the current movement base (depending on options and state)
	CharacterInputs.bUsingMovementBase = false;

	if (bUseBaseRelativeMovement)
	{
		if (const UCharacterMoverComponent* MoverComp = GetComponentByClass<UCharacterMoverComponent>())
		{
			if (UPrimitiveComponent* MovementBase = MoverComp->GetMovementBase())
			{
				FName MovementBaseBoneName = MoverComp->GetMovementBaseBoneName();

				FVector RelativeMoveInput, RelativeOrientDir;

				UBasedMovementUtils::TransformWorldDirectionToBased(MovementBase, MovementBaseBoneName, CharacterInputs.GetMoveInput(), RelativeMoveInput);
				UBasedMovementUtils::TransformWorldDirectionToBased(MovementBase, MovementBaseBoneName, CharacterInputs.OrientationIntent, RelativeOrientDir);

				CharacterInputs.SetMoveInput(CharacterInputs.GetMoveInputType(), RelativeMoveInput);
				CharacterInputs.OrientationIntent = RelativeOrientDir;

				CharacterInputs.bUsingMovementBase = true;
				CharacterInputs.MovementBase = MovementBase;
				CharacterInputs.MovementBaseBoneName = MovementBaseBoneName;
			}
		}
	}

	// Clear/consume temporal movement inputs. We are not consuming others in the event that the game world is ticking at a lower rate than the Mover simulation. 
	// In that case, we want most input to carry over between simulation frames.
	{

		bIsJumpJustPressed = false;
		bShouldToggleFlying = false;
	}
}

void AMoverExamplesCharacter::OnMoveTriggered(const FInputActionValue& Value)
{
	const FVector MovementVector = Value.Get<FVector>();
	CachedMoveInputIntent.X = FMath::Clamp(MovementVector.X, -1.0f, 1.0f);
	CachedMoveInputIntent.Y = FMath::Clamp(MovementVector.Y, -1.0f, 1.0f);
	CachedMoveInputIntent.Z = FMath::Clamp(MovementVector.Z, -1.0f, 1.0f);
}

void AMoverExamplesCharacter::OnMoveCompleted(const FInputActionValue& Value)
{
	CachedMoveInputIntent = FVector::ZeroVector;
}

void AMoverExamplesCharacter::OnLookTriggered(const FInputActionValue& Value)
{
	const FVector2D LookVector = Value.Get<FVector2D>();
	CachedLookInput.Yaw = CachedTurnInput.Yaw = FMath::Clamp(LookVector.X, -1.0f, 1.0f);
	CachedLookInput.Pitch = CachedTurnInput.Pitch = FMath::Clamp(LookVector.Y, -1.0f, 1.0f);
}

void AMoverExamplesCharacter::OnLookCompleted(const FInputActionValue& Value)
{
	CachedLookInput = FRotator::ZeroRotator;
	CachedTurnInput = FRotator::ZeroRotator;
}

void AMoverExamplesCharacter::OnJumpStarted(const FInputActionValue& Value)
{
	bIsJumpJustPressed = !bIsJumpPressed;
	bIsJumpPressed = true;
}

void AMoverExamplesCharacter::OnJumpReleased(const FInputActionValue& Value)
{
	bIsJumpPressed = false;
	bIsJumpJustPressed = false;
}

void AMoverExamplesCharacter::OnFlyTriggered(const FInputActionValue& Value)
{
	bShouldToggleFlying = true;
}
