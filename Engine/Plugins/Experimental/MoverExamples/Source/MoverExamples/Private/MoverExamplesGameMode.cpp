// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverExamplesGameMode.h"
#include "MoverExamplesGameState.h"
#include "EngineUtils.h"
#include "GameFramework/PlayerStart.h"
#include "Engine/PlayerStartPIE.h"

AMoverExamplesGameMode::AMoverExamplesGameMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GameStateClass = AMoverExamplesGameState::StaticClass();
}

AActor* AMoverExamplesGameMode::ChoosePlayerStart_Implementation(AController* Player)
{
	TArray<APlayerStart*> ClearPlayerStarts;

	APlayerStart* BestPS = nullptr;

	for (TActorIterator<APlayerStart> It(GetWorld()); It; ++It)
	{
		APlayerStart* CandidatePS = *It;
		if (CandidatePS->IsA<APlayerStartPIE>())
		{
			// Always prefer the first "Play from Here" PlayerStart, if we find one while in PIE mode
			BestPS = CandidatePS;
			break;
		}
		else
		{
			if (CanPlayerPawnFit(CandidatePS, Player))
			{
				ClearPlayerStarts.Add(CandidatePS);
			}
		}
	}

	if (BestPS == nullptr && ClearPlayerStarts.Num() > 0)
	{
		BestPS = ClearPlayerStarts[FMath::RandHelper(ClearPlayerStarts.Num())];;
	}

	// If we didn't find a good answer, fall back to the default (random choice & we may be very close to another pawn)
	return BestPS ? BestPS : Super::ChoosePlayerStart_Implementation(Player);
}



bool AMoverExamplesGameMode::CanPlayerPawnFit(const APlayerStart* SpawnPoint, const AController* Player) const
{
	APawn* MyDefaultPawn = DefaultPawnClass->GetDefaultObject<APawn>();

	static constexpr float OtherPawnCheckRadiusSq = 60.f * 60.f;

	const FVector SpawnLocation = SpawnPoint->GetActorLocation();

	for (APawn* OtherPawn : TActorRange<APawn>(GetWorld()))
	{
		if (OtherPawn != MyDefaultPawn &&
			FVector::DistSquared(SpawnLocation, OtherPawn->GetActorLocation()) < OtherPawnCheckRadiusSq)
		{
			return false;
		}
	}

	return true;
}