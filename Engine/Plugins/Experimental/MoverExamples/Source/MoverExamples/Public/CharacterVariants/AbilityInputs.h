// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AbilityInputs.generated.h"

// Data block containing extended ability inputs used by MoverExamples characters
USTRUCT(BlueprintType)
struct MOVEREXAMPLES_API FMoverExampleAbilityInputs : public FMoverDataStructBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bIsDashJustPressed = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
    bool bIsAimPressed = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bIsVaultJustPressed = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bWantsToStartZiplining = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bWantsToBeCrouched = false;

	// Implementation of FMoverDataStructBase
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override
	{
		const FMoverExampleAbilityInputs& TypedAuthority = static_cast<const FMoverExampleAbilityInputs&>(AuthorityState);
		return (TypedAuthority.bIsDashJustPressed != bIsDashJustPressed)
			|| (TypedAuthority.bIsAimPressed != bIsAimPressed)
			|| (TypedAuthority.bIsVaultJustPressed != bIsVaultJustPressed)
			|| (TypedAuthority.bWantsToStartZiplining != bWantsToStartZiplining)
			|| (TypedAuthority.bWantsToBeCrouched != bWantsToBeCrouched);
	}

	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float LerpFactor) override
	{
		const FMoverExampleAbilityInputs& FromInputs = static_cast<const FMoverExampleAbilityInputs&>(From);
		const FMoverExampleAbilityInputs& ToInputs   = static_cast<const FMoverExampleAbilityInputs&>(To);

		// "Just pressed" one-shot bools: OR across both frames so a press is never lost
		// to the interpolation cutoff. A synthesized input covering a frame where the press
		// occurred must preserve it regardless of which side of 0.5 the lerp factor lands on.
		bIsDashJustPressed     = FromInputs.bIsDashJustPressed     | ToInputs.bIsDashJustPressed;
		bIsVaultJustPressed    = FromInputs.bIsVaultJustPressed    | ToInputs.bIsVaultJustPressed;
		bWantsToStartZiplining = FromInputs.bWantsToStartZiplining | ToInputs.bWantsToStartZiplining;

		// Held/sustained bools: stable across frames, threshold selection is fine.
		const FMoverExampleAbilityInputs& SourceInputs = (LerpFactor < 0.5f) ? FromInputs : ToInputs;
		bIsAimPressed      = SourceInputs.bIsAimPressed;
		bWantsToBeCrouched = SourceInputs.bWantsToBeCrouched;
	}

	virtual void Merge(const FMoverDataStructBase& From) override
	{
		const FMoverExampleAbilityInputs& TypedFrom = static_cast<const FMoverExampleAbilityInputs&>(From);

		bIsDashJustPressed |= TypedFrom.bIsDashJustPressed;
		bIsAimPressed |= TypedFrom.bIsAimPressed;
		bIsVaultJustPressed |= TypedFrom.bIsVaultJustPressed;
		bWantsToStartZiplining |= TypedFrom.bWantsToStartZiplining;
		bWantsToBeCrouched |= TypedFrom.bWantsToBeCrouched;
	}

	// @return newly allocated copy of this FMoverExampleAbilityInputs. Must be overridden by child classes
	virtual FMoverDataStructBase* Clone() const override
	{
		// TODO: ensure that this memory allocation jives with deletion method
		FMoverExampleAbilityInputs* CopyPtr = new FMoverExampleAbilityInputs(*this);
		return CopyPtr;
	}

	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override
	{
		Super::NetSerialize(Ar, Map, bOutSuccess);

		Ar.SerializeBits(&bIsDashJustPressed, 1);
		Ar.SerializeBits(&bIsAimPressed, 1);
		Ar.SerializeBits(&bIsVaultJustPressed, 1);
		Ar.SerializeBits(&bWantsToStartZiplining, 1);
		Ar.SerializeBits(&bWantsToBeCrouched, 1);

		bOutSuccess = true;
		return true;
	}

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

	virtual void ToString(FAnsiStringBuilderBase& Out) const override
	{
		Super::ToString(Out);
		Out.Appendf("bIsDashJustPressed: %i\n", bIsDashJustPressed);
		Out.Appendf("bIsAimPressed: %i\n", bIsAimPressed);
		Out.Appendf("bIsVaultJustPressed: %i\n", bIsVaultJustPressed);
		Out.Appendf("bWantsToStartZiplining: %i\n", bWantsToStartZiplining);
		Out.Appendf("bWantsToBeCrouched: %i\n", bWantsToBeCrouched);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Super::AddReferencedObjects(Collector); }
};

UCLASS()
class MOVEREXAMPLES_API UMoverExampleAbilityInputsLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Mover|Input")
	static FMoverExampleAbilityInputs GetMoverExampleAbilityInputs(const FMoverDataCollection& FromCollection)
	{
		if (const FMoverExampleAbilityInputs* FoundInputs = FromCollection.FindDataByType<FMoverExampleAbilityInputs>())
		{
			return *FoundInputs;
		}

		return FMoverExampleAbilityInputs();
	}
};