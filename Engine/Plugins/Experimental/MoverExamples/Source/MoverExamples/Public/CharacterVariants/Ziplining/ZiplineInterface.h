// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "ZiplineInterface.generated.h"



UINTERFACE(MinimalAPI)
class UZipline : public UInterface
{
	GENERATED_BODY()
};


/** Interface that can be implemented on an object supporting simple zipline traversal between 2 points, 
    represented as Scene Components. */
class IZipline
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Zipline")
	USceneComponent* GetStartComponent();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Zipline")
	USceneComponent* GetEndComponent();
};
