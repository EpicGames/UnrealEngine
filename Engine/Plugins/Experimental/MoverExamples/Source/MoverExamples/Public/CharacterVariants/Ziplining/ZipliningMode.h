// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "ZipliningMode.generated.h"


namespace ExtendedModeNames
{
	const FName Ziplining = TEXT("Ziplining");
}


// ZipliningMode: movement mode that traverses an actor implementing the IZipline interface
UCLASS(Blueprintable, BlueprintType)
class UZipliningMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()

public:
	virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// Maximum speed 
	UPROPERTY(EditAnywhere, Category = "Ziplining", meta = (ClampMin = "1", UIMin = "1", ForceUnits = "cm/s"))
	float MaxSpeed = 1000.0f;
};


// Data block containing ziplining state info, used while ZipliningMode is active
USTRUCT()
struct FZipliningState : public FMoverDataStructBase
{
	GENERATED_USTRUCT_BODY()

	TObjectPtr<AActor> ZiplineActor;
	bool bIsMovingAtoB;

	FZipliningState()
		: bIsMovingAtoB(true)
	{
	}

	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
};

template<>
struct TStructOpsTypeTraits< FZipliningState > : public TStructOpsTypeTraitsBase2< FZipliningState >
{
	enum
	{
		WithCopy = true				// Needed for copying non-uproperty members
	};
};