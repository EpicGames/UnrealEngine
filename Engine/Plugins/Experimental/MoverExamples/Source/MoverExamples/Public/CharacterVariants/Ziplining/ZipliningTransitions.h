// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementModeTransition.h"
#include "CharacterVariants/Ziplining/ZipliningMode.h"
#include "ZipliningTransitions.generated.h"

/**
 * Transition that handles starting ziplining based on input. Character must be airborne to catch the
 * zipline, regardless of input.
 */
UCLASS(Blueprintable, BlueprintType)
class UZiplineStartTransition : public UBaseMovementModeTransition
{
	GENERATED_UCLASS_BODY()

public:
	virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;

	UPROPERTY(EditAnywhere, Category = "Ziplining")
	FName ZipliningModeName = ExtendedModeNames::Ziplining;
};

/**
 * Transition that handles exiting ziplining based on input
 */
UCLASS(Blueprintable, BlueprintType)
class UZiplineEndTransition : public UBaseMovementModeTransition
{
	GENERATED_UCLASS_BODY()

public:
	virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	// Mode to enter when exiting the zipline
	UPROPERTY(EditAnywhere, Category = "Ziplining")
	FName AutoExitToMode = DefaultModeNames::Falling;
};
