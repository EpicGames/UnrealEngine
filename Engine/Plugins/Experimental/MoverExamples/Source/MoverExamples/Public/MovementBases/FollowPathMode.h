// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "Components/InterpToMovementComponent.h"
#include "MoverTypes.h"
#include "FollowPathMode.generated.h"


/**
 * Controls how rotation is handled during pathing
 */
UENUM(BlueprintType)
enum class EFollowPathRotationType : uint8
{
	/** Maintain original actor orientation */
	Fixed,
	/** Orient moving actor by intepolating between path tangents */
	AlignWithPathTangents,
	/** Orient moving actor to the path */
	AlignWithPath,

};


/**
 * FollowPathMode: This mode performs simple movement of the associated actor, attempting to interpolate
 * through a series of locations. There are variety of settings that affect behavior, such as speed and looping.
 */
UCLASS(Blueprintable, BlueprintType)
class MOVEREXAMPLES_API UFollowPathMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()


public:
	virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	
	// List of ordered path locations to visit
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	TArray<FInterpControlPoint> ControlPoints;

	// Method of path-following
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	EInterpToBehaviourType BehaviourType;

	// Method of rotating the actor during path-following
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	EFollowPathRotationType RotationType;

	// Time (in seconds) required to move from the first point to the last (or vice versa)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing, meta = (UIMin = 0.1f, ClampMin = 0.1f, ForceUnits=s))
	float Duration = 5.0f;

protected:


	// Update the control points. Adjusts the positions of there are any actor control points as well as updating the spline type
	virtual void UpdateControlPoints(bool InForceUpdate);

	// Based on current path pct + direction + time step, find the next path pct, possibly stopping or changing direction mid-step
	float CalculateNewPathPct(float InPathPct, float InDirectionMod, float InDeltaSecs, bool& OutStopped, float& OutTimeRemainder, float& OutNewDirectionMod) const;

	// Find the necessary move delta to get onto path at a certain pct, based on current location
	FVector ComputeMoveDelta(const FVector CurrentPos, const FVector BaseLocation, const float TargetPathPos) const;

	FRotator ComputeMoveOrientation(const float TargetPathPos, const FVector& BaseLocation, FRotator DefaultOrientation) const;

	FVector ComputeInterpolatedTangentFromPathPct(const float PathPct) const;
	
	FVector ComputeTangentFromPathPct(const float PathPct, const FVector& BaseLocation) const;

#if WITH_EDITOR
	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface.
#endif // WITH_EDITOR


private:

	float TotalDistance;	// Cached distance summed between all control points
	float TimeMultiplier;	// Converts seconds to path position percent

	TArray<FVector> ControlPointPathTangents;

};


// Data block containing path-following state info
USTRUCT()
struct FFollowPathState : public FMoverDataStructBase
{
	GENERATED_USTRUCT_BODY()

	FVector BaseLocation;			// Starting point of this pathing, used for relative pathing
	float CurrentPathPos;			// [0.0, 1.0] to indicate a position on the path, as a percent from start to finish. 
	float CurrentDirectionMod;		// typically 1 or -1 to indicate direction we're traveling on the path



	FFollowPathState()
		: BaseLocation(FVector::ZeroVector)
		, CurrentPathPos(-1.0f)
		, CurrentDirectionMod(1.0f)
	{
	}

	bool HasValidPathState() const { return (CurrentPathPos >= 0.0f); }

	virtual FMoverDataStructBase* Clone() const override;


	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override
	{
		Super::NetSerialize(Ar, Map, bOutSuccess);

		Ar << BaseLocation;
		Ar << CurrentPathPos;
		Ar << CurrentDirectionMod;

		bOutSuccess = true;
		return true;
	}

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

	virtual void ToString(FAnsiStringBuilderBase& Out) const override
	{
		Super::ToString(Out);

		Out.Appendf("BaseLocation: %s\n", *BaseLocation.ToCompactString());
		Out.Appendf("CurrentPathPos: %.2f\n", CurrentPathPos);
		Out.Appendf("CurrentDirectionMod: %.1f\n", CurrentDirectionMod);
	}

	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override
	{
		const FFollowPathState* AuthoritySyncState = static_cast<const FFollowPathState*>(&AuthorityState);

		return !FVector::PointsAreSame(BaseLocation, AuthoritySyncState->BaseLocation) || 
			   !FMath::IsNearlyEqual(CurrentPathPos, AuthoritySyncState->CurrentPathPos) ||
			   !FMath::IsNearlyEqual(CurrentDirectionMod, AuthoritySyncState->CurrentDirectionMod);
	}

	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override
	{
		const FFollowPathState* FromState = static_cast<const FFollowPathState*>(&From);
		const FFollowPathState* ToState = static_cast<const FFollowPathState*>(&To);

		BaseLocation = ToState->BaseLocation;
		CurrentPathPos = FMath::Lerp(FromState->CurrentPathPos, ToState->CurrentPathPos, Pct);
		CurrentDirectionMod = ToState->CurrentDirectionMod;
	}

};

template<>
struct TStructOpsTypeTraits<FFollowPathState> : public TStructOpsTypeTraitsBase2<FFollowPathState>
{
	enum
	{
		WithCopy = true		// Necessary so that non-uproperty members get copied
	};
};