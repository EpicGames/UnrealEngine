// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "Components/InterpToMovementComponent.h"
#include "MoverTypes.h"

#include "FollowSplineMode.generated.h"

class AActor;

class USplineComponent;
class UCurveFloat;

UENUM(Blueprintable, BlueprintType)
enum class ESplineOffsetUnit : uint8
{
	Percentage UMETA(DisplayName="Percentage"),
	DurationAbsoluteSeconds UMETA(DisplayName = "Duration Absolute(sec)"),
	DistanceAbsolute UMETA(DisplayName = "Distance Absolute"),
};

UENUM(BlueprintType)
enum class EFollowSplineRotationType : uint8
{
	FollowSplineTangent UMETA(DisplayName="Follow Spline Tangent"),
	NoRotation UMETA(DisplayName = "No Rotation"),
};

/**
 * Represents an input for Range based follow behavior
 */
USTRUCT(BlueprintType)
struct FSplineOffsetRangeInput
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Inputs")
	float Value = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Inputs")
	ESplineOffsetUnit OffsetUnit = ESplineOffsetUnit::Percentage;
};

/**
 * FollowSplineMode: This mode performs movement of the associated actor, along a spline.
 * Default settings will provide a follow from start to end of the Spline. However, the start and end offsets could 
 * make the actor trace intermediate paths along the spline.
 */
UCLASS(Blueprintable, BlueprintType)
class MOVEREXAMPLES_API UFollowSplineMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()

public:
	virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UFUNCTION(BlueprintCallable, Category = "Mover|Spline")
	void SetControlSpline(const AActor* SplineProviderActor, FSplineOffsetRangeInput Offset = FSplineOffsetRangeInput());

	// Follow Mode for Path Following
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	EInterpToBehaviourType BehaviourType;

	// Rotation Mode for Path Following
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	EFollowSplineRotationType RotationType;

	// Should Mover face in the direction of movement at all times
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	bool bOrientMoverToMovement = false;

	// Should Mover follow spline with constant velocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	bool bConstantFollowVelocity = false;

	// Should the mover start following from the End
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Pathing)
	bool StartReveresed = false;

	// Optional starting offset to define ranged follow
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pathing|Offsets")
	FSplineOffsetRangeInput StartOffset;

	// Optional end offset to define ranged follow
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pathing|Offsets")
	FSplineOffsetRangeInput EndOffset;

	// If greater than zero, the follow motion would map the spline time to this duration.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pathing|Offsets", meta=(ForceUnits="sec"))
	float CustomDurationSecondsOverride = -1.0f;

	// Optional Interpolation curve to dictate the speed and position for follow
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pathing|Interpolation")
	TObjectPtr<UCurveFloat> InterpolationCurve;

protected:
	virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;

	void ConfigureSplineData();
	bool CanMove(float MeasuredSplineTime) const;

	FTransform GetTransformAtTime(float MeasuredSplineTime, const FRotator& DefaultOrientation);

	float ApplyBehaviorType(float MeasuredSplineTime);
	float ApplyFollowDirection(float MeasuredSplineTime);

	void UpdatePathState(FFollowSplineState& OutputPathState);

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Pathing)
	TObjectPtr<USplineComponent> ControlSpline;

private:
	int32 StartReversedMultiplier = 1;
	int32 FollowDirectionMultiplier = 1;
	int32 OrientationMultiplier = 1;
	
	// Local State Variables
	float CurrentSplineTime = 0.0f;
	float StartOffsetSeconds = 0.0f;
	float EndOffsetSeconds = 0.0f;
	float FollowDuration = 0.0f;

	bool bResetPingPong = false;
};


// Data block containing path-following state info
USTRUCT()
struct FFollowSplineState : public FMoverDataStructBase
{
	GENERATED_USTRUCT_BODY()

	float CurrentSplineTime;				// Current Accumulated Time on the Spline
	int32 CurrentDirectionMultiplier;		// typically 1 or -1 to indicate direction we're traveling on the path

	FFollowSplineState()
		: CurrentSplineTime(-1.0f)
		, CurrentDirectionMultiplier(1)
	{
	}

	virtual FMoverDataStructBase* Clone() const override;

	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override
	{
		Super::NetSerialize(Ar, Map, bOutSuccess);

		Ar << CurrentSplineTime;
		Ar << CurrentDirectionMultiplier;

		bOutSuccess = true;
		return true;
	}

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

	virtual void ToString(FAnsiStringBuilderBase& Out) const override
	{
		Super::ToString(Out);

		Out.Appendf("CurrentSplineTime: %.2f\n", CurrentSplineTime);
		Out.Appendf("CurrentDirectionMultiplier: %d\n", CurrentDirectionMultiplier);
	}

	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override
	{
		const FFollowSplineState* AuthoritySyncState = static_cast<const FFollowSplineState*>(&AuthorityState);

		return !FMath::IsNearlyEqual(CurrentSplineTime, AuthoritySyncState->CurrentSplineTime) ||
			   (CurrentDirectionMultiplier != AuthoritySyncState->CurrentDirectionMultiplier);
	}

	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override
	{
		const FFollowSplineState* FromState = static_cast<const FFollowSplineState*>(&From);
		const FFollowSplineState* ToState = static_cast<const FFollowSplineState*>(&To);

		CurrentSplineTime = FMath::Lerp(FromState->CurrentSplineTime, ToState->CurrentSplineTime, Pct);
		if (Pct >= 0.5f)
		{
			CurrentDirectionMultiplier = ToState->CurrentDirectionMultiplier;
		}
		else
		{
			CurrentDirectionMultiplier = FromState->CurrentDirectionMultiplier;
		}
	}
};

template<>
struct TStructOpsTypeTraits<FFollowSplineState> : public TStructOpsTypeTraitsBase2<FFollowSplineState>
{
	enum
	{
		WithCopy = true		// Necessary so that non-uproperty members get copied
	};
};
