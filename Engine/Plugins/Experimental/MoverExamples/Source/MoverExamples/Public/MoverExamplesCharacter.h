// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoverSimulationTypes.h"
#include "GameFramework/Pawn.h"
#include "MoverExamplesCharacter.generated.h"

class UNavMoverComponent;
class UInputAction;
class UCharacterMoverComponent;
struct FInputActionValue;

/** 
 * MoverExamplesCharacter: the base pawn class used by the MoverExamples plugin. Handles coalescing of input events.
 * Cannot be instantiated on its own.
 */ 

UCLASS(Abstract)
class MOVEREXAMPLES_API AMoverExamplesCharacter : public APawn, public IMoverInputProducerInterface
{
	GENERATED_BODY()

public:
	// Sets default values for this pawn's properties
	AMoverExamplesCharacter(const FObjectInitializer& ObjectInitializer);


public:
	// Called every frame
	virtual void Tick(float DeltaTime) override;

	virtual void BeginPlay() override;

	virtual void PostInitializeComponents() override;

	// Called to bind functionality to input
	virtual void SetupPlayerInputComponent(class UInputComponent* PlayerInputComponent) override;

	// Accessor for the actor's movement component
	UFUNCTION(BlueprintPure, Category = Mover)
	UCharacterMoverComponent* GetMoverComponent() const { return CharacterMotionComponent; }

	// Request the character starts moving with an intended directional magnitude. A length of 1 indicates maximum acceleration.
	UFUNCTION(BlueprintCallable, Category=MoverExamples)
	virtual void RequestMoveByIntent(const FVector& DesiredIntent) { CachedMoveInputIntent = DesiredIntent; }

	// Request the character starts moving with a desired velocity. This will be used in lieu of any other input.
	UFUNCTION(BlueprintCallable, Category=MoverExamples)
	virtual void RequestMoveByVelocity(const FVector& DesiredVelocity) { CachedMoveInputVelocity=DesiredVelocity; }

	//~ Begin INavAgentInterface Interface
	virtual FVector GetNavAgentLocation() const override;
	//~ End INavAgentInterface Interface
	
	virtual void UpdateNavigationRelevance() override;
	
protected:
	// Entry point for input production. Do not override. To extend in derived character types, override OnProduceInput for native types or implement "Produce Input" blueprint event
	virtual void ProduceInput_Implementation(int32 SimTimeMs, FMoverInputCmdContext& InputCmdResult) override;

	// Override this function in native class to author input for the next simulation frame. Consider also calling Super method.
	virtual void OnProduceInput(float DeltaMs, FMoverInputCmdContext& InputCmdResult);

	// Implement this event in Blueprints to author input for the next simulation frame. Consider also calling Parent event.
	UFUNCTION(BlueprintImplementableEvent, DisplayName="On Produce Input", meta = (ScriptName = "OnProduceInput"))
	FMoverInputCmdContext OnProduceInputInBlueprint(float DeltaMs, FMoverInputCmdContext InputCmd);

	/** Move Input Action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Input)
	TObjectPtr<UInputAction> MoveInputAction;
   
	/** Look Input Action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Input)
	TObjectPtr<UInputAction> LookInputAction;

	/** Jump Input Action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Input)
	TObjectPtr<UInputAction> JumpInputAction;

	/** Fly Input Action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Input)
	TObjectPtr<UInputAction> FlyInputAction;

public:
	// Whether or not we author our movement inputs relative to whatever base we're standing on, or leave them in world space. Only applies if standing on a base of some sort.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverExamples)
	bool bUseBaseRelativeMovement = true;
	
	/**
	 * If true, rotate the Character toward the direction the actor is moving
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverExamples)
	bool bOrientRotationToMovement = true;
	
	/**
	 * If true, the actor will continue orienting towards the last intended orientation (from input) even after movement intent input has ceased.
	 * This makes the character finish orienting after a quick stick flick from the player.  If false, character will not turn without input.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MoverExamples)
	bool bMaintainLastInputOrientation = false;

protected:
	UPROPERTY(Category = Movement, VisibleAnywhere, BlueprintReadOnly, Transient, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UCharacterMoverComponent> CharacterMotionComponent;

	/** Holds functionality for nav movement data and functions */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, Category="Nav Movement")
	TObjectPtr<UNavMoverComponent> NavMoverComponent;
	
private:
	FVector LastAffirmativeOrientationIntent = FVector::ZeroVector;	// Orientation direction at the last time we had one that wasn't zero

	FVector CachedMoveInputIntent = FVector::ZeroVector;
	FVector CachedMoveInputVelocity = FVector::ZeroVector;

	FRotator CachedTurnInput = FRotator::ZeroRotator;
	FRotator CachedLookInput = FRotator::ZeroRotator;

	bool bIsJumpJustPressed = false;
	bool bIsJumpPressed = false;
	bool bIsFlyingActive = false;
	bool bShouldToggleFlying = false;

	void OnMoveTriggered(const FInputActionValue& Value);
	void OnMoveCompleted(const FInputActionValue& Value);
	void OnLookTriggered(const FInputActionValue& Value);
	void OnLookCompleted(const FInputActionValue& Value);
	void OnJumpStarted(const FInputActionValue& Value);
	void OnJumpReleased(const FInputActionValue& Value);
	void OnFlyTriggered(const FInputActionValue& Value);

	uint8 bHasProduceInputinBpFunc : 1;
};
