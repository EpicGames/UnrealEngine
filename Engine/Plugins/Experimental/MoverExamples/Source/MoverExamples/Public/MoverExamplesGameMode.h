// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/GameMode.h"
#include "Engine/Engine.h"
#include "MoverExamplesGameMode.generated.h"

class APlayerStart;

UCLASS(BlueprintType)
class AMoverExamplesGameMode : public AGameMode
{
	GENERATED_BODY()

	AMoverExamplesGameMode(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool AllowCheats(APlayerController* P) { return true; }

	/** select best spawn point for player */
	virtual AActor* ChoosePlayerStart_Implementation(AController* Player) override;

private:
	bool CanPlayerPawnFit(const APlayerStart* SpawnPoint, const AController* Player) const;
};