// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/GameState.h"
#include "Engine/Engine.h"
#include "MoverExamplesGameState.generated.h"

UCLASS(BlueprintType)
class MOVEREXAMPLES_API AMoverExamplesGameState : public AGameState
{
	GENERATED_BODY()

	AMoverExamplesGameState() = default;
};